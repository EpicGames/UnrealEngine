// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationWidgets : ModuleRules
{
	public AnimationWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		/** NOTE: THIS MODULE SHOULD NOT EVER DEPEND ON UNREALED. 
		 * Please refer to ToolWidgets.Build.cs for more information.
		 */
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Slate",
				"SlateCore",
				"InputCore",
				"ToolWidgets",
				"AnimationCore",
			}
		);

		if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"CoreUObject",
					"UnrealEd",
					"GraphEditor",
					"PropertyEditor"
				}
			);
		}
	}
}
