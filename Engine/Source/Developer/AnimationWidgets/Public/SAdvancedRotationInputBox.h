// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimationCoreLibrary.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/NumericUnitTypeInterface.inl"

namespace ESlateRotationRepresentation
{
	enum Type : int
	{
		EulerXYZ,
		EulerXZY,
		EulerYXZ,
		EulerYZX,
		EulerZXY,
		EulerZYX,
		Rotator,
		Quaternion,
		AxisAndAngle,
		Max
	};
}

namespace ESlateTransformSubComponent
{
	enum Type : int
	{
		X,
		Y,
		Z,
		W,
		Angle,
		Pitch,
		Yaw,
		Roll,
		Max
	};
}

/**
 * Generic Rotation Slate control
 */
template<typename NumericType = FVector::FReal>
class SAdvancedRotationInputBox : public SCompoundWidget
{
public:
	
	typedef SNumericVectorInputBox<NumericType, UE::Math::TVector<NumericType>, 3> SNumericVectorInputBox3; 
	typedef SNumericVectorInputBox<NumericType, UE::Math::TVector4<NumericType>, 4> SNumericVectorInputBox4; 

	/** Delegate for notification for a rotator value change */
 	DECLARE_DELEGATE_OneParam(FOnRotatorChanged, FRotator);

	/** Delegate for notification for a rotator value commit */
	DECLARE_DELEGATE_TwoParams(FOnRotatorCommitted, FRotator, ETextCommit::Type);

	/** Delegate for notification for a quaternion value change */
	DECLARE_DELEGATE_OneParam(FOnQuaternionChanged, FQuat);

	/** Delegate for notification for a quaternion value commit */
	DECLARE_DELEGATE_TwoParams(FOnQuaternionCommitted, FQuat, ETextCommit::Type);

	/** Delegate to retrieve a numerictype value  */
	DECLARE_DELEGATE_RetVal_TwoParams(TOptional<NumericType>, FOnGetNumericValue, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for numerictype value change */
 	DECLARE_DELEGATE_ThreeParams(FOnNumericValueChanged, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType);

	/** Notification for numerictype value committed */
	DECLARE_DELEGATE_FourParams(FOnNumericValueCommitted, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType, ETextCommit::Type);

	/** Notification for begin slider movement */
	DECLARE_DELEGATE_TwoParams(FOnBeginSliderMovement, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for end slider movement */
	DECLARE_DELEGATE_ThreeParams(FOnEndSliderMovement, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType);

	/** Delegate to retrieve toggle checkstate */
	DECLARE_DELEGATE_RetVal_TwoParams(ECheckBoxState, FOnGetToggleChecked, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for toggle checkstate change */
	DECLARE_DELEGATE_ThreeParams(FOnToggleChanged, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, ECheckBoxState);

	SLATE_BEGIN_ARGS( SAdvancedRotationInputBox )
		: _Representation(ESlateRotationRepresentation::Rotator)
		, _bColorAxisLabels(true)
		, _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
		, _AllowSpin(true)
		, _DisplayToggle( false )
		, _TogglePadding( FMargin( 1.f,0.f,1.f,0.f ) )
		, _PreventThrottling(false)
		{}

		/** Representation of the rotation */
		SLATE_ATTRIBUTE( ESlateRotationRepresentation::Type, Representation )

		/** Optional representation as rotator */
		SLATE_ATTRIBUTE( TOptional<FRotator>, Rotator )

		/** Optional delegate to notify a change in the rotator */
		SLATE_EVENT( FOnRotatorChanged, OnRotatorChanged )

		/** Optional delegate to notify a commit in the rotator */
		SLATE_EVENT( FOnRotatorCommitted, OnRotatorCommitted )

		/** Optional representation as quaternion */
		SLATE_ATTRIBUTE( TOptional<FQuat>, Quaternion )

		/** Optional delegate to notify a change in the quaternion */
		SLATE_EVENT( FOnQuaternionChanged, OnQuaternionChanged )

		/** Optional delegate to notify a commit in the quaternion */
		SLATE_EVENT( FOnQuaternionCommitted, OnQuaternionCommitted )

		/** Optional delegate to retrieve a value */
		SLATE_EVENT( FOnGetNumericValue, OnGetNumericValue )

		/** Optional delegate to notify a value change */
		SLATE_EVENT( FOnNumericValueChanged, OnNumericValueChanged )

		/** Optional delegate to notify a committed value */
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericValueCommitted )

		/** Optional delegate to notify a begin slider movement */
		SLATE_EVENT(FOnBeginSliderMovement, OnBeginSliderMovement)

		/** Optional delegate to notify a end slider movement */
		SLATE_EVENT(FOnEndSliderMovement, OnEndSliderMovement)

		/** Should the axis labels be colored */
		SLATE_ARGUMENT( bool, bColorAxisLabels )		

		/** Font to use for the text in this box */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Whether or not values can be spun or if they should be typed in */
		SLATE_ARGUMENT( bool, AllowSpin )

		/** Whether or not to include a toggle checkbox to the left of the widget */
		SLATE_ARGUMENT( bool, DisplayToggle )

		/** Delegate to retrieve a toggle state */
		SLATE_EVENT( FOnGetToggleChecked, OnGetToggleChecked )

		/** Delegate to notify a toggle state change */
		SLATE_EVENT( FOnToggleChanged, OnToggleChanged )

		/** Padding around the toggle checkbox */
		SLATE_ARGUMENT( FMargin, TogglePadding )

		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs )
	{
		Representation = InArgs._Representation;
		Rotator = InArgs._Rotator;
		OnRotatorChanged = InArgs._OnRotatorChanged;
		OnRotatorCommitted = InArgs._OnRotatorCommitted;
		Quaternion = InArgs._Quaternion;
		OnQuaternionChanged = InArgs._OnQuaternionChanged;
		OnQuaternionCommitted = InArgs._OnQuaternionCommitted;
		OnGetNumericValue = InArgs._OnGetNumericValue; 
		OnNumericValueChanged = InArgs._OnNumericValueChanged; 
		OnNumericValueCommitted = InArgs._OnNumericValueCommitted; 
		OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
		OnEndSliderMovement = InArgs._OnEndSliderMovement;
		OnGetToggleChecked = InArgs._OnGetToggleChecked; 
		OnToggleChanged = InArgs._OnToggleChanged;

		TSharedRef<SWidget> AngleLabelWidget = SNullWidget::NullWidget;
		if (InArgs._bColorAxisLabels)
		{
			const FLinearColor AngleLabelColor = SNumericEntryBox<NumericType>::GreenLabelBackgroundColor;
			AngleLabelWidget = SNumericEntryBox<NumericType>::BuildNarrowColorLabel(AngleLabelColor);
		}

		TSharedPtr<TNumericUnitTypeInterface<NumericType>> DegreesTypeInterface =
			MakeShareable( new TNumericUnitTypeInterface<NumericType>(EUnit::Degrees) );

		this->ChildSlot
		[
			SNew(SHorizontalBox)

			// a rotator variation - which is the default in the editor
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.f))
			[
				SNew(SNumericRotatorInputBox<NumericType>)
				.bColorAxisLabels(InArgs._bColorAxisLabels)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.TypeInterface(DegreesTypeInterface)
				.Roll(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Roll)
				.Yaw(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Yaw)
				.Pitch(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Pitch)
				.OnRollChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Roll)
				.OnYawChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Yaw)
				.OnPitchChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Pitch)
				.OnRollCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Roll)
				.OnYawCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Yaw)
				.OnPitchCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Pitch)
				.DisplayToggle(InArgs._DisplayToggle)
				.ToggleRollChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Roll)
				.ToggleYawChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Yaw)
				.TogglePitchChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Pitch)
				.OnToggleRollChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Roll)
				.OnToggleYawChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Yaw)
				.OnTogglePitchChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Pitch)
				.OnPitchBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Pitch)
				.OnYawBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Yaw)
				.OnRollBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Roll)
				.OnPitchEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Pitch)
				.OnYawEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Yaw)
				.OnRollEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Roll)
				.TogglePadding(InArgs._TogglePadding)
				.Visibility(this, &SAdvancedRotationInputBox::IsRotatorInputBoxVisible)
				.PreventThrottling(InArgs._PreventThrottling)
			]

			// a vector 3 variation to represent euler with rotation order
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.f))
			[
				SNew(SNumericVectorInputBox3)
				.bColorAxisLabels(InArgs._bColorAxisLabels)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.SpinDelta(0.01f)
				.MinSliderVector(FVector(-360.f, -360.f, -360.f))
				.MaxSliderVector(FVector(360.f, 360.f, 360.f))
				.TypeInterface(DegreesTypeInterface)
				.X(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::X)
				.Y(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Y)
				.Z(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Z)
				.OnXChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::X)
				.OnYChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Y)
				.OnZChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Z)
				.OnXCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::X)
				.OnYCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Y)
				.OnZCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Z)
				.DisplayToggle(InArgs._DisplayToggle)
				.ToggleXChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::X)
				.ToggleYChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Y)
				.ToggleZChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Z)
				.OnToggleXChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::X)
				.OnToggleYChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Y)
				.OnToggleZChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Z)
				.OnXBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::X)
				.OnXEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::X)
				.OnYBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Y)
				.OnYEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Y)
				.OnZBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Z)
				.OnZEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Z)
				.TogglePadding(InArgs._TogglePadding)
				.Visibility(this, &SAdvancedRotationInputBox::IsEulerInputBoxVisible)
				.PreventThrottling(InArgs._PreventThrottling)
			]
			
			// a vector 4d variation - to represent quaternion
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(FMargin(0.f))
			[
				SNew(SNumericVectorInputBox4)
				.bColorAxisLabels(InArgs._bColorAxisLabels)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.SpinDelta(0.001f)
				.MinSliderVector(FVector4(-1.f, -1.f, -1.f, -1.f))
				.MaxSliderVector(FVector4(1.f, 1.f, 1.f, 1.f))
				.X(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::X)
				.Y(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Y)
				.Z(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Z)
				.W(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::W)
				.OnXChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::X)
				.OnYChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Y)
				.OnZChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Z)
				.OnWChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::W)
				.OnXCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::X)
				.OnYCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Y)
				.OnZCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Z)
				.OnWCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::W)
				.DisplayToggle(InArgs._DisplayToggle)
				.ToggleXChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::X)
				.ToggleYChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Y)
				.ToggleZChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Z)
				.ToggleWChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::W)
				.OnToggleXChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::X)
				.OnToggleYChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Y)
				.OnToggleZChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Z)
				.OnToggleWChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::W)
				.OnXBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::X)
				.OnXEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::X)
				.OnYBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Y)
				.OnYEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Y)
				.OnZBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Z)
				.OnZEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Z)
				.OnWBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::W)
				.OnWEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::W)
				.TogglePadding(InArgs._TogglePadding)
				.Visibility(this, &SAdvancedRotationInputBox::IsQuaternionInputBoxVisible)
				.PreventThrottling(InArgs._PreventThrottling)
			]

			/*
			// a numeric input for the angle for the axis and angle representation
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.f))
			.FillWidth(0.333f)
			[
				SNew(SNumericEntryBox<NumericType>)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.Delta(0.01f)
				.MinSliderValue(-360.f)

				.Visibility(this, &SAdvancedRotationInputBox::IsAxisAndAngleInputBoxVisible)
			]
			*/

			// a vector 3 variation to represent the axis for axis and angle
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.f))
			[
				SNew(SNumericVectorInputBox3)
				.bColorAxisLabels(InArgs._bColorAxisLabels)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.SpinDelta(0.01f)
				.MinSliderVector(FVector(-1.f, -1.f, -1.f))
				.MaxSliderVector(FVector(1.f, 1.f, 1.f))
				.X(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::X)
				.Y(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Y)
				.Z(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Z)
				.OnXChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::X)
				.OnYChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Y)
				.OnZChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Z)
				.OnXCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::X)
				.OnYCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Y)
				.OnZCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Z)
				.DisplayToggle(InArgs._DisplayToggle)
				.ToggleXChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::X)
				.ToggleYChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Y)
				.ToggleZChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Z)
				.OnToggleXChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::X)
				.OnToggleYChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Y)
				.OnToggleZChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Z)
				.OnXBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::X)
				.OnXEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::X)
				.OnYBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Y)
				.OnYEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Y)
				.OnZBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Z)
				.OnZEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Z)
				.TogglePadding(InArgs._TogglePadding)
				.Visibility(this, &SAdvancedRotationInputBox::IsAxisAndAngleInputBoxVisible)
				.PreventThrottling(InArgs._PreventThrottling)
			]

			// a numeric input for the angle for the axis and angle representation
			+SHorizontalBox::Slot()
			.Padding(FMargin(0.f))
			.FillWidth(0.333f)
			[
				SNew(SNumericEntryBox<NumericType>)
				.Font(InArgs._Font)
				.AllowSpin(InArgs._AllowSpin)
				.Delta(0.01f)
				.MinSliderValue(-360.f)
				.MaxSliderValue(360.f)
				.Value(this, &SAdvancedRotationInputBox::HandleGetNumericValue, ESlateTransformSubComponent::Angle)
				.OnValueChanged(this, &SAdvancedRotationInputBox::HandleOnNumericValueChanged, ESlateTransformSubComponent::Angle)
				.OnValueCommitted(this, &SAdvancedRotationInputBox::HandleOnNumericValueCommitted, ESlateTransformSubComponent::Angle)
				.ToolTipText_Lambda([this]()
				{
					TOptional<NumericType> Value = HandleGetNumericValue(ESlateTransformSubComponent::Angle);
					if (Value.IsSet())
					{
						return FText::Format(NSLOCTEXT("SAdvancedRotationInputBox", "Angle_ToolTip", "Angle: {0}"), Value.GetValue());
					}
					return NSLOCTEXT("SAdvancedRotationInputBox", "MultipleValues", "Multiple Values");
				})
				.UndeterminedString(NSLOCTEXT("SAdvancedRotationInputBox", "MultipleValues", "Multiple Values"))
				.TypeInterface(DegreesTypeInterface)
				.LabelPadding(FMargin(3))
				.LabelLocation(SNumericEntryBox<NumericType>::ELabelLocation::Inside)
				.Label()
				[
					AngleLabelWidget
				]
				.DisplayToggle(InArgs._DisplayToggle)
				.TogglePadding(InArgs._TogglePadding)
				.ToggleChecked(this, &SAdvancedRotationInputBox::HandleGetToggleChecked, ESlateTransformSubComponent::Angle)
				.OnToggleChanged(this, &SAdvancedRotationInputBox::HandleOnToggleChanged, ESlateTransformSubComponent::Angle)
				.OnBeginSliderMovement(this, &SAdvancedRotationInputBox::HandleOnBeginSliderMovement, ESlateTransformSubComponent::Angle)
				.OnEndSliderMovement(this, &SAdvancedRotationInputBox::HandleOnEndSliderMovement, ESlateTransformSubComponent::Angle)
				.Visibility(this, &SAdvancedRotationInputBox::IsAxisAndAngleInputBoxVisible)
				.PreventThrottling(InArgs._PreventThrottling)
			]
		];

	}

	static bool IsValidComponent(ESlateRotationRepresentation::Type InRepresentation, ESlateTransformSubComponent::Type InSubComponent)
	{
		switch(InRepresentation)
		{
			case ESlateRotationRepresentation::EulerXYZ:
			case ESlateRotationRepresentation::EulerXZY:
			case ESlateRotationRepresentation::EulerYXZ:
			case ESlateRotationRepresentation::EulerYZX:
			case ESlateRotationRepresentation::EulerZXY:
			case ESlateRotationRepresentation::EulerZYX:
			{
				switch(InSubComponent)
				{
					case ESlateTransformSubComponent::X:
					case ESlateTransformSubComponent::Y:
					case ESlateTransformSubComponent::Z:
					{
						return true;
					}
				}
				break;
			}
			case ESlateRotationRepresentation::Rotator:
			{
				switch(InSubComponent)
				{
					case ESlateTransformSubComponent::Roll:
					case ESlateTransformSubComponent::Yaw:
					case ESlateTransformSubComponent::Pitch:
					{
						return true;
					}
				}
				break;
			}
			case ESlateRotationRepresentation::Quaternion:
			{
				switch(InSubComponent)
				{
					case ESlateTransformSubComponent::X:
					case ESlateTransformSubComponent::Y:
					case ESlateTransformSubComponent::Z:
					case ESlateTransformSubComponent::W:
					{
						return true;
					}
				}
				break;
			}
			case ESlateRotationRepresentation::AxisAndAngle:
			{
				switch(InSubComponent)
				{
					case ESlateTransformSubComponent::X:
					case ESlateTransformSubComponent::Y:
					case ESlateTransformSubComponent::Z:
					case ESlateTransformSubComponent::Angle:
					{
						return true;
					}
				}
				break;
			}
		}

		return false;
	}

private:

	TAttribute<ESlateRotationRepresentation::Type> Representation;
	TAttribute<TOptional<FRotator>> Rotator;
	FOnRotatorChanged OnRotatorChanged;
	FOnRotatorCommitted OnRotatorCommitted;
	TAttribute<TOptional<FQuat>> Quaternion;
	FOnQuaternionChanged OnQuaternionChanged;
	FOnQuaternionCommitted OnQuaternionCommitted;
	FOnGetNumericValue OnGetNumericValue;
	FOnNumericValueChanged OnNumericValueChanged;
	FOnNumericValueCommitted OnNumericValueCommitted;
	FOnBeginSliderMovement OnBeginSliderMovement;
	FOnEndSliderMovement OnEndSliderMovement;
	FOnGetToggleChecked OnGetToggleChecked;
	FOnToggleChanged OnToggleChanged;

	TOptional<NumericType> HandleGetNumericValue(ESlateTransformSubComponent::Type InSubComponent) const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get(); 
		if(IsValidComponent(Repr, InSubComponent))
		{
			if(OnGetNumericValue.IsBound())
			{
				return OnGetNumericValue.Execute(Repr, InSubComponent);
			}
			else if(Rotator.IsBound() || Quaternion.IsBound())
			{
				TOptional<FRotator> Rot;
				TOptional<FQuat> Quat;
				GetRotatorAndQuat(Rot, Quat);

				if(Rot.IsSet() && Quat.IsSet())
				{
					switch(Repr)
					{
						case ESlateRotationRepresentation::EulerXYZ:
						case ESlateRotationRepresentation::EulerXZY:
						case ESlateRotationRepresentation::EulerYXZ:
						case ESlateRotationRepresentation::EulerYZX:
						case ESlateRotationRepresentation::EulerZXY:
						case ESlateRotationRepresentation::EulerZYX:
						{
								const int32 RotationOrder = int32(Repr) - int32(ESlateRotationRepresentation::EulerXYZ);
								const FVector Euler = AnimationCore::EulerFromQuat(Quat.GetValue(), EEulerRotationOrder(RotationOrder), true);
								switch(InSubComponent)
								{
									case ESlateTransformSubComponent::X:
									{
										return Euler.X;
									}
									case ESlateTransformSubComponent::Y:
									{
										return Euler.Y;
									}
									case ESlateTransformSubComponent::Z:
									{
										return Euler.Z;
									}
									default:
									{
										break;
									}
								}
								break;
						}
						case ESlateRotationRepresentation::Rotator:
						{
							switch(InSubComponent)
							{
								case ESlateTransformSubComponent::Pitch:
								{
									return Rot->Pitch;
								}
								case ESlateTransformSubComponent::Yaw:
								{
									return Rot->Yaw;
								}
								case ESlateTransformSubComponent::Roll:
								{
									return Rot->Roll;
								}
								default:
								{
									break;
								}
							}
							break;
						}
						case ESlateRotationRepresentation::Quaternion:
						{
							switch(InSubComponent)
							{
								case ESlateTransformSubComponent::X:
								{
									return Quat->X;
								}
								case ESlateTransformSubComponent::Y:
								{
									return Quat->Y;
								}
								case ESlateTransformSubComponent::Z:
								{
									return Quat->Z;
								}
								case ESlateTransformSubComponent::W:
								{
									return Quat->W;
								}
								default:
								{
									break;
								}
							}
							break;
						}
						case ESlateRotationRepresentation::AxisAndAngle:
						{
							FVector Axis;
							double Angle;
							Quat->ToAxisAndAngle(Axis, Angle);
							switch(InSubComponent)
							{
								case ESlateTransformSubComponent::X:
								{
									return Axis.X;
								}
								case ESlateTransformSubComponent::Y:
								{
									return Axis.Y;
								}
								case ESlateTransformSubComponent::Z:
								{
									return Axis.Z;
								}
								case ESlateTransformSubComponent::Angle:
								{
									return Angle;
								}
								default:
								{
									break;
								}
							}
							break;
						}
					}
				}
			}
		}
		return TOptional<NumericType>();
	}

	bool GetRotatorAndQuat(TOptional<FRotator>& Rot, TOptional<FQuat> &Quat) const
	{
		bool bSuccess = false;
		if(Rotator.IsBound())
		{
			Rot = Rotator.Get();
			if(Rot.IsSet())
			{
				Quat = Rot->Quaternion();
				bSuccess = true;
			}
		}
				
		if(Quaternion.IsBound())
		{
			Quat = Quaternion.Get();
			if(Quat.IsSet())
			{
				if(!Rot.IsSet())
				{
					Rot = Quat->Rotator();
				}
				bSuccess = true;;
			}
			else if(Rot.IsSet())
			{
				Quat = Rot->Quaternion();
				bSuccess = true;
			}
		}
		return bSuccess;
	}

	bool ApplyNumericValueChangeToRotatorAndQuat(TOptional<FRotator>& Rot, TOptional<FQuat> &Quat, NumericType InValue, ESlateTransformSubComponent::Type InSubComponent) const
	{
		if(!GetRotatorAndQuat(Rot, Quat))
		{
			return false;
		}
		
		if(Rot.IsSet() && Quat.IsSet())
		{
			const ESlateRotationRepresentation::Type& Repr = Representation.Get();
			switch(Repr)
			{
				case ESlateRotationRepresentation::EulerXYZ:
				case ESlateRotationRepresentation::EulerXZY:
				case ESlateRotationRepresentation::EulerYXZ:
				case ESlateRotationRepresentation::EulerYZX:
				case ESlateRotationRepresentation::EulerZXY:
				case ESlateRotationRepresentation::EulerZYX:
				{
						const int32 RotationOrder = int32(Repr) - int32(ESlateRotationRepresentation::EulerXYZ);
						FVector Euler = AnimationCore::EulerFromQuat(Quat.GetValue(), EEulerRotationOrder(RotationOrder), true);
						switch(InSubComponent)
						{
							case ESlateTransformSubComponent::X:
							{
								Euler.X = InValue;
								break;
							}
							case ESlateTransformSubComponent::Y:
							{
								Euler.Y = InValue;
								break;
							}
							case ESlateTransformSubComponent::Z:
							{
								Euler.Z = InValue;
								break;
							}
							default:
							{
								break;
							}
						}
						Quat = AnimationCore::QuatFromEuler(Euler, EEulerRotationOrder(RotationOrder), true);
						Rot = Quat->Rotator();
						break;
				}
				case ESlateRotationRepresentation::Rotator:
				{
					switch(InSubComponent)
					{
						case ESlateTransformSubComponent::Pitch:
						{
							Rot->Pitch = InValue;
							break;
						}
						case ESlateTransformSubComponent::Yaw:
						{
							Rot->Yaw = InValue;
							break;
						}
						case ESlateTransformSubComponent::Roll:
						{
							Rot->Roll = InValue;
							break;
						}
						default:
						{
							break;
						}
					}
					Quat = Rot->Quaternion();
					break;
				}
				case ESlateRotationRepresentation::Quaternion:
				{
					switch(InSubComponent)
					{
						case ESlateTransformSubComponent::X:
						{
							Quat->X = InValue;
							break;
						}
						case ESlateTransformSubComponent::Y:
						{
							Quat->Y = InValue;
							break;
						}
						case ESlateTransformSubComponent::Z:
						{
							Quat->Z = InValue;
							break;
						}
						case ESlateTransformSubComponent::W:
						{
							Quat->W = InValue;
							break;
						}
						default:
						{
							break;
						}
					}
					Rot = Quat->Rotator();
					break;
				}
				case ESlateRotationRepresentation::AxisAndAngle:
				{
					FVector Axis;
					double Angle;
					Quat->ToAxisAndAngle(Axis, Angle);
					switch(InSubComponent)
					{
						case ESlateTransformSubComponent::X:
						{
							Axis.X = InValue;
							break;
						}
						case ESlateTransformSubComponent::Y:
						{
							Axis.Y = InValue;
							break;
						}
						case ESlateTransformSubComponent::Z:
						{
							Axis.Z = InValue;
							break;
						}
						case ESlateTransformSubComponent::Angle:
						{
							Angle = InValue;
							break;
						}
						default:
						{
							break;
						}
					}

					Quat = FQuat(Axis, Angle);
					Rot = Quat->Rotator();
					break;
				}
			}
		}

		return true;
	}

	void HandleOnNumericValueChanged(NumericType InValue, ESlateTransformSubComponent::Type InSubComponent)
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get(); 
		if(IsValidComponent(Repr, InSubComponent))
		{
			if(OnNumericValueChanged.IsBound())
			{
				OnNumericValueChanged.Execute(Repr, InSubComponent, InValue);
			}

			if(OnRotatorChanged.IsBound() || OnQuaternionChanged.IsBound())
			{
				TOptional<FRotator> Rot;
				TOptional<FQuat> Quat;
				if(ApplyNumericValueChangeToRotatorAndQuat(Rot, Quat, InValue, InSubComponent))
				{
					if(OnRotatorChanged.IsBound() && Rot.IsSet())
					{
						OnRotatorChanged.Execute(Rot.GetValue());
					}
					if(OnQuaternionChanged.IsBound() && Quat.IsSet())
					{
						OnQuaternionChanged.Execute(Quat.GetValue());
					}
				}
			}
		}
	}
	
	void HandleOnNumericValueCommitted(NumericType InValue, ETextCommit::Type InCommitType, ESlateTransformSubComponent::Type InSubComponent)
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get(); 
		if(IsValidComponent(Repr, InSubComponent))
		{
			if(OnNumericValueCommitted.IsBound())
			{
				OnNumericValueCommitted.Execute(Repr, InSubComponent, InValue, InCommitType);
			}
			if(OnRotatorCommitted.IsBound() || OnQuaternionCommitted.IsBound())
			{
				TOptional<FRotator> Rot;
				TOptional<FQuat> Quat;
				if(ApplyNumericValueChangeToRotatorAndQuat(Rot, Quat, InValue, InSubComponent))
				{
					if(OnRotatorCommitted.IsBound() && Rot.IsSet())
					{
						OnRotatorCommitted.Execute(Rot.GetValue(), InCommitType);
					}
					if(OnQuaternionCommitted.IsBound() && Quat.IsSet())
					{
						OnQuaternionCommitted.Execute(Quat.GetValue(), InCommitType);
					}
				}
			}
		}
	}

	ECheckBoxState HandleGetToggleChecked(ESlateTransformSubComponent::Type InSubComponent) const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get(); 
		if(IsValidComponent(Repr, InSubComponent))
		{
			if(OnGetToggleChecked.IsBound())
			{
				return OnGetToggleChecked.Execute(Repr, InSubComponent);
			}
		}
		return ECheckBoxState::Checked;
	}

	void HandleOnToggleChanged(ECheckBoxState InValue, ESlateTransformSubComponent::Type InSubComponent)
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get(); 
		if(IsValidComponent(Repr, InSubComponent))
		{
			if(OnToggleChanged.IsBound())
			{
				OnToggleChanged.Execute(Repr, InSubComponent, InValue);
			}
		}
	}

	void HandleOnBeginSliderMovement(ESlateTransformSubComponent::Type InSubComponent)
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		if(IsValidComponent(Repr, InSubComponent))
		{
			if (OnBeginSliderMovement.IsBound())
			{
				OnBeginSliderMovement.Execute(Repr, InSubComponent);
			}
		}
	}
	
	void HandleOnEndSliderMovement(NumericType InValue, ESlateTransformSubComponent::Type InSubComponent)
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		if(IsValidComponent(Repr, InSubComponent))
		{
			if (OnEndSliderMovement.IsBound())
			{
				OnEndSliderMovement.Execute(Repr, InSubComponent, InValue);
			}
		}
	}

	EVisibility IsRotatorInputBoxVisible() const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		return Repr == ESlateRotationRepresentation::Rotator ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility IsEulerInputBoxVisible() const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		switch(Repr)
		{
			case ESlateRotationRepresentation::EulerXYZ:
			case ESlateRotationRepresentation::EulerXZY:
			case ESlateRotationRepresentation::EulerYXZ:
			case ESlateRotationRepresentation::EulerYZX:
			case ESlateRotationRepresentation::EulerZXY:
			case ESlateRotationRepresentation::EulerZYX:
			{
				return EVisibility::Visible;
			}
		}
		return EVisibility::Collapsed;
	}

	EVisibility IsQuaternionInputBoxVisible() const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		switch(Repr)
		{
			case ESlateRotationRepresentation::Quaternion:
			{
				return EVisibility::Visible;
			}
		}
		return EVisibility::Collapsed;
	}

	EVisibility IsAxisAndAngleInputBoxVisible() const
	{
		const ESlateRotationRepresentation::Type& Repr = Representation.Get();
		switch(Repr)
		{
			case ESlateRotationRepresentation::AxisAndAngle:
			{
				return EVisibility::Visible;
			}
		}
		return EVisibility::Collapsed;
	}
};

