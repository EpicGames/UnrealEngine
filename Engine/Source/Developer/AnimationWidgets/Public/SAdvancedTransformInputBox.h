// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad include

#include "SAdvancedRotationInputBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"

#if WITH_EDITOR

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#endif

namespace ESlateTransformComponent
{
	enum Type : int
	{
		Location,
		Rotation,
		Scale,
		Max
	};
}

/**
 * Generic Transform Slate control
 */
template<typename TransformType = FTransform, typename NumericType = FVector::FReal>
class SAdvancedTransformInputBox : public SCompoundWidget
{
public:

	typedef SNumericVectorInputBox<NumericType, UE::Math::TVector<NumericType>, 3> SNumericVectorInputBox3; 

	/** Delegate for notification for a transform value change */
	DECLARE_DELEGATE_OneParam(FOnTransformChanged, TransformType);

	/** Delegate for notification for a transform value commit */
	DECLARE_DELEGATE_TwoParams(FOnTransformCommitted, TransformType, ETextCommit::Type);

	/** Delegate to retrieve a numerictype value  */
	DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<NumericType>, FOnGetNumericValue, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for numerictype value change */
 	DECLARE_DELEGATE_FourParams(FOnNumericValueChanged, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType);

	/** Notification for numerictype value committed */
	DECLARE_DELEGATE_FiveParams(FOnNumericValueCommitted, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType, ETextCommit::Type);

	/** Notification for begin slider movement */
	DECLARE_DELEGATE_ThreeParams(FOnBeginSliderMovement, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for end slider movement */
	DECLARE_DELEGATE_FourParams(FOnEndSliderMovement, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, NumericType);

	/** Delegate to retrieve relative / world state */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetIsComponentRelative, ESlateTransformComponent::Type);

	/** Notification for a relative / world state change */
	DECLARE_DELEGATE_TwoParams(FOnIsComponentRelativeChanged, ESlateTransformComponent::Type, bool /* is relative? */);

	/** Notification for the rotation representation change */
	DECLARE_DELEGATE_OneParam(FOnRotationRepresentationChanged, ESlateRotationRepresentation::Type);

	/** Notification for the scale lock change */
	DECLARE_DELEGATE_OneParam(FOnScaleLockChanged, bool /* is locked? */);

	/** Delegate to retrieve toggle checkstate */
	DECLARE_DELEGATE_RetVal_ThreeParams(ECheckBoxState, FOnGetToggleChecked, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type);

	/** Notification for toggle checkstate change */
	DECLARE_DELEGATE_FourParams(FOnToggleChanged, ESlateTransformComponent::Type, ESlateRotationRepresentation::Type, ESlateTransformSubComponent::Type, ECheckBoxState);

	/** Delegate to fire when a row or the whole transform (type == max) is being copied to the clipboard */
	DECLARE_DELEGATE_OneParam(FOnCopyToClipboard, ESlateTransformComponent::Type);

	/** Delegate to fire when a row or the whole transform (type == max) is being pasted into from the clipboard */
	DECLARE_DELEGATE_OneParam(FOnPasteFromClipboard, ESlateTransformComponent::Type);

	/** Delegate to fire when a row or the whole transform (type == max) needs to be reset to its default */
	DECLARE_DELEGATE_OneParam(FOnResetToDefault, ESlateTransformComponent::Type);

	/** Delegate to determine if the row or the whole transform (type == max) differs from its default */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FDiffersFromDefault, ESlateTransformComponent::Type);

	SLATE_BEGIN_ARGS( SAdvancedTransformInputBox )
		: _ConstructLocation(true)
		, _ConstructRotation(true)
		, _ConstructScale(true)
		, _UseQuaternionForRotation(false)
		, _RotationRepresentation(TSharedPtr<ESlateRotationRepresentation::Type>( new ESlateRotationRepresentation::Type(ESlateRotationRepresentation::Rotator)))
		, _bColorAxisLabels(true)
		, _ShowInlineLabels(false) 
		, _LocationLabel(NSLOCTEXT("SAdvancedTransformInputBox", "Location", "Location"))
		, _RotationLabel(NSLOCTEXT("SAdvancedTransformInputBox", "Rotation", "Rotation"))
		, _ScaleLabel(NSLOCTEXT("SAdvancedTransformInputBox", "Scale", "Scale"))
		, _LabelPadding( FMargin( 0.f,0.f,6.f,0.f ) )
		, _Font(FAppStyle::Get().GetFontStyle("NormalFont"))
		, _AllowSpin(true)
		, _LocationSpinDelta(0.01f)
		, _ScaleSpinDelta(0.001f)
		, _AllowEditRotationRepresentation(true)
		, _DisplayScaleLock(true)
		, _IsScaleLocked(TSharedPtr<bool>(new bool(false)))
		, _DisplayRelativeWorld(false)
		, _DisplayToggle( false )
		, _TogglePadding( FMargin( 1.f,0.f,1.f,0.f ) )
		, _PreventThrottling(false)
		{}

		/** Whether or not to construct the location component */
		SLATE_ARGUMENT( bool, ConstructLocation )

		/** Whether or not to construct the rotation component */
		SLATE_ARGUMENT( bool, ConstructRotation )

		/** Whether or not to construct the scale component */
		SLATE_ARGUMENT( bool, ConstructScale )

		/** Whether to use the quaternion is the primary rotation representation or the rotator (default) */
		SLATE_ARGUMENT( bool, UseQuaternionForRotation )

		/** Optional representation as transform */
		SLATE_ATTRIBUTE( TOptional<TransformType>, Transform )

		/** Optional delegate to notify a change in the transform */
		SLATE_EVENT( FOnTransformChanged, OnTransformChanged )

		/** Optional delegate to notify a commit in the transform */
		SLATE_EVENT( FOnTransformCommitted, OnTransformCommitted )

		/** Representation of the rotation */
		SLATE_ARGUMENT( TSharedPtr<ESlateRotationRepresentation::Type>, RotationRepresentation )

		/** Delegate to notify a rotation representation change */
		SLATE_EVENT( FOnRotationRepresentationChanged, OnRotationRepresentationChanged )

		/** Delegate to retrieve a value */
		SLATE_EVENT( FOnGetNumericValue, OnGetNumericValue )

		/** Delegate to notify a value change */
		SLATE_EVENT( FOnNumericValueChanged, OnNumericValueChanged )

		/** Delegate to notify a committed value */
		SLATE_EVENT( FOnNumericValueCommitted, OnNumericValueCommitted )

		/** Delegate to notify a begin slider movement */
		SLATE_EVENT(FOnBeginSliderMovement, OnBeginSliderMovement)

		/** Delegate to notify a end slider movement */
		SLATE_EVENT(FOnEndSliderMovement, OnEndSliderMovement)

		/** Should the axis labels be colored */
		SLATE_ARGUMENT( bool, bColorAxisLabels )		

		/** Should we show labels inline next to all input boxes */
		SLATE_ARGUMENT( bool, ShowInlineLabels )		

		/** The label to use for the location component */
		SLATE_ARGUMENT( FText, LocationLabel )		

		/** The label to use for the rotation component */
		SLATE_ARGUMENT( FText, RotationLabel )		

		/** The label to use for the scale component */
		SLATE_ARGUMENT( FText, ScaleLabel )		

		/** Padding around the label checkbox */
		SLATE_ARGUMENT( FMargin, LabelPadding )

		/** Font to use for the text in this box */
		SLATE_ATTRIBUTE( FSlateFontInfo, Font )

		/** Whether or not values can be spun or if they should be typed in */
		SLATE_ARGUMENT( bool, AllowSpin )

		/** The delta amount to apply, per pixel, when a location spinner is dragged. */
		SLATE_ATTRIBUTE( NumericType, LocationSpinDelta )

		/** The delta amount to apply, per pixel, when a scale spinner is dragged. */
		SLATE_ATTRIBUTE( NumericType, ScaleSpinDelta )

		/** Whether or not to display the rotation representation picker in the label */
		SLATE_ARGUMENT( bool, AllowEditRotationRepresentation )
	
		/** Whether or not to display the lock scale button */
		SLATE_ARGUMENT( bool, DisplayScaleLock )

		/** The state of the scale lock */
		SLATE_ARGUMENT( TSharedPtr<bool>, IsScaleLocked )

		/** Delegate to notify a scale lock change */
		SLATE_EVENT( FOnScaleLockChanged, OnScaleLockChanged )

		/** Whether or not to display the per component buttons to switch between relative and world */
		SLATE_ARGUMENT( bool, DisplayRelativeWorld )

		/** Delegate to retrieve a relative / world state per component */
		SLATE_EVENT( FOnGetIsComponentRelative, OnGetIsComponentRelative )

		/** Delegate to notify a relative / world state change */
		SLATE_EVENT( FOnIsComponentRelativeChanged, OnIsComponentRelativeChanged )

		/** Whether or not to include a toggle checkbox to the left of the widget */
		SLATE_ARGUMENT( bool, DisplayToggle )

		/** Delegate to retrieve a toggle state */
		SLATE_EVENT( FOnGetToggleChecked, OnGetToggleChecked )

		/** Delegate to notify a toggle state change */
		SLATE_EVENT( FOnToggleChanged, OnToggleChanged )

		/** Padding around the toggle checkbox */
		SLATE_ARGUMENT( FMargin, TogglePadding )

		/** Delegate to react to copy to clipboard */
		SLATE_EVENT( FOnCopyToClipboard, OnCopyToClipboard )

		/** Delegate to react to paste from clipboard */
		SLATE_EVENT( FOnPasteFromClipboard, OnPasteFromClipboard )

		/** Delegate to react to reset to default */
		SLATE_EVENT( FOnResetToDefault, OnResetToDefault )

		/** Delegate to determine if a row (or the whole transform) differs from its default */
		SLATE_EVENT( FDiffersFromDefault, DiffersFromDefault )

		/** If refresh requests for the viewport should happen for all value changes **/
		SLATE_ARGUMENT(bool, PreventThrottling)
	
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs )
	{
		TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

		if(InArgs._ConstructLocation)
		{
			VerticalBox->AddSlot()
			[
				ConstructWidget(InArgs, ESlateTransformComponent::Location)
			];
		}
		if(InArgs._ConstructRotation)
		{
			VerticalBox->AddSlot()
			[
				ConstructWidget(InArgs, ESlateTransformComponent::Rotation)
			];
		}
		if(InArgs._ConstructScale)
		{
			VerticalBox->AddSlot()
			[
				ConstructWidget(InArgs, ESlateTransformComponent::Scale)
			];
		}

		ChildSlot
		[
			VerticalBox
		];
	}

	/**
	 * Construct an input widget
	 */
	static TSharedRef<SWidget> ConstructWidget(const FArguments& InArgs, ESlateTransformComponent::Type InComponent)
	{
		SHorizontalBox::FArguments BoxArgs;
		((FSlateBaseNamedArgs&)BoxArgs) = (FSlateBaseNamedArgs)InArgs;
		TSharedRef<SHorizontalBox> HorizontalBox = SArgumentNew(BoxArgs, SHorizontalBox);

		if(InArgs._ShowInlineLabels)
		{
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(InArgs._LabelPadding)
			.AutoWidth()
			[
				ConstructLabel(InArgs, InComponent)
			];
		}

		TSharedRef<SWidget> InputWidget = SNullWidget::NullWidget;
		TAttribute<TOptional<TransformType>> Transform = InArgs._Transform;
		FOnTransformChanged OnTransformChanged = InArgs._OnTransformChanged;
		FOnTransformCommitted OnTransformCommitted = InArgs._OnTransformCommitted;
		FOnGetNumericValue OnGetNumericValue = InArgs._OnGetNumericValue;
		FOnNumericValueChanged OnNumericValueChanged = InArgs._OnNumericValueChanged;
		FOnNumericValueCommitted OnNumericValueCommitted = InArgs._OnNumericValueCommitted;
		FOnBeginSliderMovement OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
		FOnEndSliderMovement OnEndSliderMovement = InArgs._OnEndSliderMovement;
		FOnGetToggleChecked OnGetToggleChecked = InArgs._OnGetToggleChecked;
		FOnToggleChanged OnToggleChanged = InArgs._OnToggleChanged;
		FOnRotationRepresentationChanged OnRotationRepresentationChanged = InArgs._OnRotationRepresentationChanged;
		const bool UseQuaternionForRotation = InArgs._UseQuaternionForRotation;

		auto OnGetLocation = [Transform, OnGetNumericValue]() -> TOptional<FVector>
		{
			if(Transform.IsBound())
			{
				TOptional<TransformType> Xfo = Transform.Get();
				if(Xfo.IsSet())
				{
					return Xfo->GetLocation();
				}
			}
			if(OnGetNumericValue.IsBound())
			{
				FVector Location = FVector::OneVector;
				Location.X = OnGetNumericValue.Execute(ESlateTransformComponent::Location, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X).Get(Location.X);
				Location.Y = OnGetNumericValue.Execute(ESlateTransformComponent::Location, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y).Get(Location.Y);
				Location.Z = OnGetNumericValue.Execute(ESlateTransformComponent::Location, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z).Get(Location.Z);
				return Location;
			}
			return TOptional<FVector>();
		};

		auto OnGetRotator = [Transform, OnGetNumericValue]() -> TOptional<FRotator>
		{
			if(Transform.IsBound())
			{
				TOptional<TransformType> Xfo = Transform.Get();
				if(Xfo.IsSet())
				{
					return Xfo->Rotator();
				}
			}
			if(OnGetNumericValue.IsBound())
			{
				FRotator Rotator = FRotator::ZeroRotator;
				Rotator.Roll = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Rotator, ESlateTransformSubComponent::Roll).Get(Rotator.Roll);
				Rotator.Pitch = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Rotator, ESlateTransformSubComponent::Pitch).Get(Rotator.Pitch);
				Rotator.Yaw = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Rotator, ESlateTransformSubComponent::Yaw).Get(Rotator.Yaw);
				return Rotator;
			}
			return TOptional<FRotator>();
		};

		auto OnGetQuaternion = [Transform, OnGetNumericValue]() -> TOptional<FQuat>
		{
			if(Transform.IsBound())
			{
				TOptional<TransformType> Xfo = Transform.Get();
				if(Xfo.IsSet())
				{
					return Xfo->GetRotation().GetNormalized();
				}
			}
			if(OnGetNumericValue.IsBound())
			{
				FQuat Quat = FQuat::Identity;
				Quat.X = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Quaternion, ESlateTransformSubComponent::X).Get(Quat.X);
				Quat.Y = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Quaternion, ESlateTransformSubComponent::Y).Get(Quat.Y);
				Quat.Z = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Quaternion, ESlateTransformSubComponent::Z).Get(Quat.Z);
				Quat.W = OnGetNumericValue.Execute(ESlateTransformComponent::Rotation, ESlateRotationRepresentation::Quaternion, ESlateTransformSubComponent::W).Get(Quat.W);
				return Quat.GetNormalized();
			}
			return TOptional<FQuat>();
		};

		auto OnGetScale = [Transform, OnGetNumericValue]() -> TOptional<FVector>
		{
			if(Transform.IsBound())
			{
				TOptional<TransformType> Xfo = Transform.Get();
				if(Xfo.IsSet())
				{
					return Xfo->GetScale3D();
				}
			}
			if(OnGetNumericValue.IsBound())
			{
				FVector Scale = FVector::OneVector;
				Scale.X = OnGetNumericValue.Execute(ESlateTransformComponent::Scale, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X).Get(Scale.X);
				Scale.Y = OnGetNumericValue.Execute(ESlateTransformComponent::Scale, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y).Get(Scale.Y);
				Scale.Z = OnGetNumericValue.Execute(ESlateTransformComponent::Scale, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z).Get(Scale.Z);
				return Scale;
			}
			return TOptional<FVector>();
		};

		auto OnGetVector = [OnGetLocation, OnGetScale](ESlateTransformComponent::Type InComponent) -> TOptional<FVector>
		{
			return (InComponent == ESlateTransformComponent::Location) ? OnGetLocation() : OnGetScale();
		};

		auto OnGetTransform = [OnGetLocation, OnGetQuaternion, OnGetRotator, OnGetScale, UseQuaternionForRotation]() -> TransformType
		{
			TransformType Result = TransformType::Identity;
			Result.SetLocation(OnGetLocation().Get(Result.GetLocation()));
			Result.SetScale3D(OnGetScale().Get(Result.GetScale3D()));

			if(UseQuaternionForRotation)
			{
				Result.SetRotation(OnGetQuaternion().Get(Result.GetRotation()));
			}
			else
			{
				const FRotator Rotator = OnGetRotator().Get(Result.Rotator());
				Result = TransformType(Rotator, Result.GetLocation(), Result.GetScale3D());
			}
			Result.NormalizeRotation();
			return Result;
		};

		switch(InComponent)
		{
			case ESlateTransformComponent::Location:
			case ESlateTransformComponent::Scale:
			{
				TAttribute<TOptional<NumericType>> XAttribute, YAttribute, ZAttribute;
				TAttribute<TOptional<FVector>> Vector3Attribute;

				if(InArgs._OnGetNumericValue.IsBound())
				{
					XAttribute = TAttribute<TOptional<NumericType>>::CreateLambda(
						[OnGetNumericValue, OnGetVector, InComponent]() -> TOptional<NumericType>
						{
							return OnGetNumericValue.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X);
						}); 
					YAttribute = TAttribute<TOptional<NumericType>>::CreateLambda(
						[OnGetNumericValue, OnGetVector, InComponent]() -> TOptional<NumericType>
						{
							return OnGetNumericValue.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y);
						}); 
					ZAttribute = TAttribute<TOptional<NumericType>>::CreateLambda(
						[OnGetNumericValue, OnGetVector, InComponent]() -> TOptional<NumericType>
						{
							return OnGetNumericValue.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z);
						}); 
				}

				if(InArgs._Transform.IsBound() || InArgs._Transform.IsSet())
				{
					Vector3Attribute = TAttribute<TOptional<FVector>>::CreateLambda(
						[OnGetVector, InComponent]() -> TOptional<FVector>
						{
							return OnGetVector(InComponent);
						});
				}

				typename SNumericVectorInputBox3::FOnNumericValueChanged XChanged, YChanged, ZChanged;
				if(OnNumericValueChanged.IsBound())
				{
					XChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X, InValue);
						}
					); 
					YChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y, InValue);
						}
					); 
					ZChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z, InValue);
						}
					); 
				}

				FSimpleDelegate XBeginSlide, YBeginSlide, ZBeginSlide;
				if(OnBeginSliderMovement.IsBound())
				{
					XBeginSlide = FSimpleDelegate::CreateLambda(
						[OnBeginSliderMovement, InComponent]()
						{
							OnBeginSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X);
						}
					); 
					YBeginSlide = FSimpleDelegate::CreateLambda(
						[OnBeginSliderMovement, InComponent]()
						{
							OnBeginSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y);
						}
					); 
					ZBeginSlide = FSimpleDelegate::CreateLambda(
						[OnBeginSliderMovement, InComponent]()
						{
							OnBeginSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z);
						}
					); 
				}

				typename SNumericVectorInputBox3::FOnNumericValueChanged XEndSlide, YEndSlide, ZEndSlide;
				if(OnEndSliderMovement.IsBound())
				{
					XEndSlide = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnEndSliderMovement, InComponent](NumericType InValue)
						{
							OnEndSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X, InValue);
						}
					); 
					YEndSlide = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnEndSliderMovement, InComponent](NumericType InValue)
						{
							OnEndSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y, InValue);
						}
					); 
					ZEndSlide = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnEndSliderMovement, InComponent](NumericType InValue)
						{
							OnEndSliderMovement.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z, InValue);
						}
					); 
				}

				typename SNumericVectorInputBox3::FOnVectorValueChanged VectorChanged;
				if(OnTransformChanged.IsBound())
				{
					VectorChanged = SNumericVectorInputBox3::FOnVectorValueChanged::CreateLambda(
						[OnTransformChanged, OnGetTransform, InComponent](FVector InValue)
						{
							if(OnTransformChanged.IsBound())
							{
								TransformType Transform = OnGetTransform();
								if(InComponent == ESlateTransformComponent::Location)
								{
									Transform.SetLocation(InValue);
								}
								else
								{
									Transform.SetScale3D(InValue);
								}
								OnTransformChanged.Execute(Transform);
							}
						}
					);
				}

				typename SNumericVectorInputBox3::FOnNumericValueCommitted XCommitted, YCommitted, ZCommitted;
				if(OnNumericValueCommitted.IsBound())
				{
					XCommitted = SNumericVectorInputBox3::FOnNumericValueCommitted::CreateLambda(
						[OnNumericValueCommitted, InComponent](NumericType InValue, ETextCommit::Type InCommitType)
						{
							OnNumericValueCommitted.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X, InValue, InCommitType);
						}
					); 
					YCommitted = SNumericVectorInputBox3::FOnNumericValueCommitted::CreateLambda(
						[OnNumericValueCommitted, InComponent](NumericType InValue, ETextCommit::Type InCommitType)
						{
							OnNumericValueCommitted.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y, InValue, InCommitType);
						}
					); 
					ZCommitted = SNumericVectorInputBox3::FOnNumericValueCommitted::CreateLambda(
						[OnNumericValueCommitted, InComponent](NumericType InValue, ETextCommit::Type InCommitType)
						{
							OnNumericValueCommitted.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z, InValue, InCommitType);
						}
					); 
				}
					
				typename SNumericVectorInputBox3::FOnVectorValueCommitted VectorCommitted;
				if(OnTransformCommitted.IsBound())
				{
					VectorCommitted = SNumericVectorInputBox3::FOnVectorValueCommitted::CreateLambda(
						[OnTransformCommitted, OnGetTransform, InComponent](FVector InValue, ETextCommit::Type InCommitType)
						{
							if(OnTransformCommitted.IsBound())
							{
								TransformType Transform = OnGetTransform();
								if(InComponent == ESlateTransformComponent::Location)
								{
									Transform.SetLocation(InValue);
								}
								else
								{
									Transform.SetScale3D(InValue);
								}
								OnTransformCommitted.Execute(Transform, InCommitType);
							}
						}
					);
				}

				TSharedPtr<bool> ScaleLockState = InArgs._IsScaleLocked;
				typename SNumericVectorInputBox3::FOnConstrainVector ConstrainComponents;
					
				if(InComponent == ESlateTransformComponent::Scale)
				{
					if(ScaleLockState.IsValid())
					{
						ConstrainComponents = SNumericVectorInputBox3::FOnConstrainVector::CreateStatic(
							&SAdvancedTransformInputBox::ConstrainScale, TAttribute<bool>::CreateLambda([ScaleLockState]() 
							{
								return *ScaleLockState.Get();
							})
						);
					}
				}

				InputWidget = SNew(SNumericVectorInputBox3)
					.Font(InArgs._Font)
					.AllowSpin(InArgs._AllowSpin)
					.SpinDelta(InComponent == ESlateTransformComponent::Location ? InArgs._LocationSpinDelta : InArgs._ScaleSpinDelta)
					.bColorAxisLabels(InArgs._bColorAxisLabels)
					.X(XAttribute)
					.Y(YAttribute)
					.Z(ZAttribute)
					.Vector(Vector3Attribute)
					.OnXChanged(XChanged)
					.OnYChanged(YChanged)
					.OnZChanged(ZChanged)
					.OnVectorChanged(VectorChanged)
					.OnXCommitted(XCommitted)
					.OnYCommitted(YCommitted)
					.OnZCommitted(ZCommitted)
					.OnVectorCommitted(VectorCommitted)
					.ConstrainVector(ConstrainComponents)
					.DisplayToggle(InArgs._DisplayToggle)
					.TogglePadding(InArgs._TogglePadding)
					.PreventThrottling(InArgs._PreventThrottling)
					.OnXBeginSliderMovement(XBeginSlide)
					.OnYBeginSliderMovement(YBeginSlide)
					.OnZBeginSliderMovement(ZBeginSlide)
					.OnXEndSliderMovement(XEndSlide)
					.OnYEndSliderMovement(YEndSlide)
					.OnZEndSliderMovement(ZEndSlide)
					.ToggleXChecked_Lambda([OnGetToggleChecked, InComponent]() -> ECheckBoxState
					{
						if(OnGetToggleChecked.IsBound())
						{
							return OnGetToggleChecked.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X);
						}
						return ECheckBoxState::Checked;
					})
					.ToggleYChecked_Lambda([OnGetToggleChecked, InComponent]() -> ECheckBoxState
					{
						if(OnGetToggleChecked.IsBound())
						{
							return OnGetToggleChecked.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y);
						}
						return ECheckBoxState::Checked;
					})
					.ToggleZChecked_Lambda([OnGetToggleChecked, InComponent]() -> ECheckBoxState
					{
						if(OnGetToggleChecked.IsBound())
						{
							return OnGetToggleChecked.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z);
						}
						return ECheckBoxState::Checked;
					})
					.OnToggleXChanged_Lambda([OnToggleChanged, InComponent](ECheckBoxState InState)
					{
						if(OnToggleChanged.IsBound())
						{
							OnToggleChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X, InState);
						}
					})
					.OnToggleYChanged_Lambda([OnToggleChanged, InComponent](ECheckBoxState InState)
					{
						if(OnToggleChanged.IsBound())
						{
							OnToggleChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y, InState);
						}
					})
					.OnToggleZChanged_Lambda([OnToggleChanged, InComponent](ECheckBoxState InState)
					{
						if(OnToggleChanged.IsBound())
						{
							OnToggleChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z, InState);
						}
					});
					
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				typename SAdvancedRotationInputBox<NumericType>::FOnGetNumericValue RotationGetNumericValue;
				if(OnGetNumericValue.IsBound())
				{
					RotationGetNumericValue = SAdvancedRotationInputBox<NumericType>::FOnGetNumericValue::CreateLambda(
						[OnGetNumericValue, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent) -> TOptional<NumericType>
					{
						return OnGetNumericValue.Execute(InComponent, InRepr, InSubComponent);
					});
				}

				typename SAdvancedRotationInputBox<NumericType>::FOnNumericValueChanged RotationValueChanged;
				if(OnNumericValueChanged.IsBound())
				{
					RotationValueChanged = SAdvancedRotationInputBox<NumericType>::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent, NumericType InValue)
					{
						OnNumericValueChanged.Execute(InComponent, InRepr, InSubComponent, InValue);
					});
				}

				typename SAdvancedRotationInputBox<NumericType>::FOnNumericValueCommitted RotationValueCommitted;
				if(OnNumericValueCommitted.IsBound())
				{
					RotationValueCommitted = SAdvancedRotationInputBox<NumericType>::FOnNumericValueCommitted::CreateLambda(
						[OnNumericValueCommitted, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent, NumericType InValue, ETextCommit::Type InCommitType)
					{
						OnNumericValueCommitted.Execute(InComponent, InRepr, InSubComponent, InValue, InCommitType);
					});
				}

				typename SAdvancedRotationInputBox<NumericType>::FOnBeginSliderMovement RotationBeginSliderMovement;
				if(OnBeginSliderMovement.IsBound())
				{
					RotationBeginSliderMovement = SAdvancedRotationInputBox<NumericType>::FOnBeginSliderMovement::CreateLambda(
						[OnBeginSliderMovement, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent)
					{
						OnBeginSliderMovement.Execute(InComponent, InRepr, InSubComponent);
					});
				}

				typename SAdvancedRotationInputBox<NumericType>::FOnEndSliderMovement RotationEndSliderMovement;
				if(OnEndSliderMovement.IsBound())
				{
					RotationEndSliderMovement = SAdvancedRotationInputBox<NumericType>::FOnEndSliderMovement::CreateLambda(
						[OnEndSliderMovement, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent, NumericType InValue)
					{
						OnEndSliderMovement.Execute(InComponent, InRepr, InSubComponent, InValue);
					});
				}

				TAttribute<TOptional<FQuat>> QuaternionAttribute;
				TAttribute<TOptional<FRotator>> RotatorAttribute;
				if(InArgs._Transform.IsBound() || InArgs._Transform.IsSet())
				{
					if(InArgs._UseQuaternionForRotation)
					{
						QuaternionAttribute = TAttribute<TOptional<FQuat>>::CreateLambda(
							[OnGetQuaternion]() -> TOptional<FQuat>
							{
								return OnGetQuaternion();
							});
					}
					else
					{
						RotatorAttribute = TAttribute<TOptional<FRotator>>::CreateLambda(
							[OnGetRotator]() -> TOptional<FRotator>
							{
								return OnGetRotator();
							});
					}
				}

				typename SNumericVectorInputBox3::FOnNumericValueChanged XChanged, YChanged, ZChanged;
				if(OnNumericValueChanged.IsBound())
				{
					XChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::X, InValue);
						}
					); 
					YChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Y, InValue);
						}
					); 
					ZChanged = SNumericVectorInputBox3::FOnNumericValueChanged::CreateLambda(
						[OnNumericValueChanged, InComponent](NumericType InValue)
						{
							OnNumericValueChanged.Execute(InComponent, ESlateRotationRepresentation::EulerXYZ, ESlateTransformSubComponent::Z, InValue);
						}
					); 
				}
					
				typename SAdvancedRotationInputBox<NumericType>::FOnRotatorChanged RotatorChanged;
				typename SAdvancedRotationInputBox<NumericType>::FOnQuaternionChanged QuaternionChanged;
				if(OnTransformChanged.IsBound())
				{
					if(InArgs._UseQuaternionForRotation)
					{
						QuaternionChanged = SAdvancedRotationInputBox<NumericType>::FOnQuaternionChanged::CreateLambda(
							[OnTransformChanged, OnGetTransform](FQuat InQuaternion)
							{
								TransformType Xfo = OnGetTransform();
								Xfo.SetRotation(InQuaternion);
								OnTransformChanged.Execute(Xfo);
							}
						);
					}
					else
					{
						RotatorChanged = SAdvancedRotationInputBox<NumericType>::FOnRotatorChanged::CreateLambda(
							[OnTransformChanged, OnGetTransform](FRotator InRotator)
							{
								TransformType Xfo = OnGetTransform();
								Xfo = TransformType(InRotator, Xfo.GetLocation(), Xfo.GetScale3D());
								OnTransformChanged.Execute(Xfo);
							}
						);
					}
				}

				typename SAdvancedRotationInputBox<NumericType>::FOnRotatorCommitted RotatorCommitted;
				typename SAdvancedRotationInputBox<NumericType>::FOnQuaternionCommitted QuaternionCommitted;
				if(OnTransformCommitted.IsBound())
				{
					if(InArgs._UseQuaternionForRotation)
					{
						QuaternionCommitted = SAdvancedRotationInputBox<NumericType>::FOnQuaternionCommitted::CreateLambda(
							[OnTransformCommitted, OnGetTransform](FQuat InQuaternion, ETextCommit::Type InCommitType)
							{
								TransformType Xfo = OnGetTransform();
								Xfo.SetRotation(InQuaternion);
								OnTransformCommitted.Execute(Xfo, InCommitType);
							}
						);
					}
					else
					{
						RotatorCommitted = SAdvancedRotationInputBox<NumericType>::FOnRotatorCommitted::CreateLambda(
							[OnTransformCommitted, OnGetTransform](FRotator InRotator, ETextCommit::Type InCommitType)
							{
								TransformType Xfo = OnGetTransform();
								Xfo = TransformType(InRotator, Xfo.GetLocation(), Xfo.GetScale3D());
								OnTransformCommitted.Execute(Xfo, InCommitType);
							}
						);
					}
				}

				TSharedPtr<ESlateRotationRepresentation::Type> RotationRepresentationPtr = InArgs._RotationRepresentation;

				InputWidget = SNew(SAdvancedRotationInputBox<NumericType>)
					.Font(InArgs._Font)
					.AllowSpin(InArgs._AllowSpin)
					.bColorAxisLabels(InArgs._bColorAxisLabels)
					.Representation_Lambda([RotationRepresentationPtr]() -> ESlateRotationRepresentation::Type
					{
						if(RotationRepresentationPtr.IsValid())
						{
							return *RotationRepresentationPtr.Get();
						}
						return ESlateRotationRepresentation::Rotator;
					})
					.OnGetNumericValue(RotationGetNumericValue)
					.OnNumericValueChanged(RotationValueChanged)
					.OnNumericValueCommitted(RotationValueCommitted)
					.Rotator(RotatorAttribute)
					.OnRotatorChanged(RotatorChanged)
					.OnRotatorCommitted(RotatorCommitted)
					.Quaternion(QuaternionAttribute)
					.OnQuaternionChanged(QuaternionChanged)
					.OnQuaternionCommitted(QuaternionCommitted)
					.DisplayToggle(InArgs._DisplayToggle)
					.TogglePadding(InArgs._TogglePadding)
					.PreventThrottling(InArgs._PreventThrottling)
					.OnBeginSliderMovement(RotationBeginSliderMovement)
					.OnEndSliderMovement(RotationEndSliderMovement)
					.OnGetToggleChecked(SAdvancedRotationInputBox<NumericType>::FOnGetToggleChecked::CreateLambda(
						[OnGetToggleChecked, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent) -> ECheckBoxState
					{
						if(OnGetToggleChecked.IsBound())
						{
							return OnGetToggleChecked.Execute(InComponent, InRepr, InSubComponent);
						}
						return ECheckBoxState::Checked;
					}))
					.OnToggleChanged(SAdvancedRotationInputBox<NumericType>::FOnToggleChanged::CreateLambda(
						[OnToggleChanged, InComponent](ESlateRotationRepresentation::Type InRepr, ESlateTransformSubComponent::Type InSubComponent, ECheckBoxState InState)
					{
						if(OnToggleChanged.IsBound())
						{
							return OnToggleChanged.Execute(InComponent, InRepr, InSubComponent, InState);
						}
					}));

				break;
			}
		}

		if(InputWidget != SNullWidget::NullWidget)
		{
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.FillWidth(1.f)
			[
				InputWidget
			];
		}

		return HorizontalBox;
	}

	/**
	 * Constructs a label widget
	 */
	static TSharedRef<SWidget> ConstructLabel(const FArguments& InArgs, ESlateTransformComponent::Type InComponent)
	{
		const TAttribute<bool> EnabledAttribute = InArgs._IsEnabled;

		TSharedRef<SWidget> LabelWidget = SNullWidget::NullWidget;

		if(InComponent == ESlateTransformComponent::Rotation && InArgs._AllowEditRotationRepresentation)
		{
			const TArray<FText>& Labels = GetRotationRepresentationLabels();

			TSharedPtr<ESlateRotationRepresentation::Type> RotationRepresentationPtr = InArgs._RotationRepresentation;
			FOnRotationRepresentationChanged OnRotationRepresentationChanged = InArgs._OnRotationRepresentationChanged;

			LabelWidget = SNew(SComboButton)
				.IsEnabled(true)
				.ContentPadding(0.f)
				.OnGetMenuContent_Lambda([Labels, RotationRepresentationPtr, OnRotationRepresentationChanged]
				{
					FMenuBuilder MenuBuilder(true, nullptr);
					for (int32 LabelIndex = 0; LabelIndex < Labels.Num(); LabelIndex++)
					{
						MenuBuilder.AddMenuEntry(
							Labels[LabelIndex],
							FText(),
							FSlateIcon(),
							FUIAction
							(
								FExecuteAction::CreateLambda([LabelIndex, RotationRepresentationPtr, OnRotationRepresentationChanged]()
								{
									if(OnRotationRepresentationChanged.IsBound())
									{
										OnRotationRepresentationChanged.Execute((ESlateRotationRepresentation::Type)LabelIndex);
									}
									if(RotationRepresentationPtr.IsValid())
									{
										*RotationRepresentationPtr.Get() = (ESlateRotationRepresentation::Type)LabelIndex;
									}
								}),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([LabelIndex, RotationRepresentationPtr]() -> bool
								{
									if(RotationRepresentationPtr.IsValid())
									{
										return LabelIndex == (int32)*RotationRepresentationPtr.Get(); 
									}
									return false;
								})
							),
							NAME_None,
							EUserInterfaceActionType::Check);
					}
					return MenuBuilder.MakeWidget();
				})
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(InArgs._Font)
					.Text_Lambda([RotationRepresentationPtr, Labels]() -> FText
					{
						if(RotationRepresentationPtr.IsValid())
						{
							return Labels[(int32)*RotationRepresentationPtr.Get()];
						}
						return Labels[0];
					})
				];
		}
		else
		{
			const FText& LabelText =
				(InComponent == ESlateTransformComponent::Rotation) ?
					InArgs._RotationLabel : (
				(InComponent == ESlateTransformComponent::Location) ?
					InArgs._LocationLabel :
					InArgs._ScaleLabel);

			LabelWidget = SNew(STextBlock)
			.IsEnabled(EnabledAttribute)
			.Font(InArgs._Font)
			.Text(LabelText);
		}

		SHorizontalBox::FArguments BoxArgs;
		((FSlateBaseNamedArgs&)BoxArgs) = (FSlateBaseNamedArgs)InArgs;
		BoxArgs.IsEnabled(true);
		TSharedRef<SHorizontalBox> HorizontalBox = SArgumentNew(BoxArgs, SHorizontalBox);

		HorizontalBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			LabelWidget
		];

		const TSharedRef<SWidget> ScaleLockWidget = ConstructScaleLockWidget(InArgs, InComponent);
		if(ScaleLockWidget != SNullWidget::NullWidget)
		{
			ScaleLockWidget->SetEnabled(EnabledAttribute);
			
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				ScaleLockWidget
			];
		}

		const TSharedRef<SWidget> RelativeWorldWidget = ConstructRelativeWorldWidget(InArgs, InComponent);
		if(RelativeWorldWidget != SNullWidget::NullWidget)
		{
			// the relative to world will be always enabled...
			// (unless the outer scope if not enabled)
			
			HorizontalBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(4.f, 0.f, 0.f, 0.f))
			[
				RelativeWorldWidget
			];
		}

		return HorizontalBox;
	}

	/**
	 * Constructs a relative / world switcher button
	 */
	static TSharedRef<SWidget> ConstructScaleLockWidget(const FArguments& InArgs, ESlateTransformComponent::Type InComponent )
	{
		if(InArgs._DisplayScaleLock && InArgs._ConstructScale && InComponent == ESlateTransformComponent::Scale && InArgs._IsScaleLocked.IsValid())
		{
			TSharedPtr<bool> LockState = InArgs._IsScaleLocked;
			FOnScaleLockChanged OnScaleLockChanged = InArgs._OnScaleLockChanged;

			static const FText TooltipText = NSLOCTEXT("SAdvancedTransformInputBox", "PreserveScaleToolTip",
					 "When locked, all axis values scale together so the object maintains its proportions in all directions.");

			return SNew(SButton)
				.ContentPadding(0.f)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([OnScaleLockChanged, LockState]() -> FReply
				{
					*LockState.Get() = !*LockState.Get();
					if(OnScaleLockChanged.IsBound())
					{
						OnScaleLockChanged.Execute(*LockState.Get());
					}					
					return FReply::Handled();
				})
				.ToolTipText(TooltipText)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor(FSlateColor::UseStyle())
				.Content()
				[
					SNew(SImage)
					.Image_Lambda([LockState]() -> const FSlateBrush*
					{
						return GetScaleLockIcon(*LockState.Get());
					})
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}
		return SNullWidget::NullWidget;
	}

	/**
	 * Gather the icon for scale locked / unlocked
	 */
	static const FSlateBrush* GetScaleLockIcon(bool bIsLocked)
	{
		return bIsLocked ? FAppStyle::Get().GetBrush( TEXT("Icons.Lock") ) : FAppStyle::Get().GetBrush( TEXT("Icons.Unlock") ) ;
	}
	
	/**
	 * Constructs a relative / world switcher button
	 */
	static TSharedRef<SWidget> ConstructRelativeWorldWidget(const FArguments& InArgs, ESlateTransformComponent::Type InComponent)
	{
		if(InArgs._DisplayRelativeWorld)
		{
			FOnGetIsComponentRelative OnGetIsComponentRelative = InArgs._OnGetIsComponentRelative;;
			FOnIsComponentRelativeChanged OnIsComponentRelativeChanged = InArgs._OnIsComponentRelativeChanged;

			static const FText TooltipFormat = NSLOCTEXT("SAdvancedTransformInputBox", "RelativeWorldTooltip",
					"Cycles the Transform {0} coordinate system between world and local (relative) space.\nShift clicking this button affects the whole transform.");
			static const FText LocationTooltip = FText::Format(TooltipFormat, NSLOCTEXT("SAdvancedTransformInputBox", "Location", "Location"));
			static const FText RotationTooltip = FText::Format(TooltipFormat, NSLOCTEXT("SAdvancedTransformInputBox", "Rotation", "Rotation"));
			static const FText ScaleTooltip = FText::Format(TooltipFormat, NSLOCTEXT("SAdvancedTransformInputBox", "Scale", "Scale"));

			FText TooltipText;
			switch(InComponent)
			{
				case ESlateTransformComponent::Location:
				{
					TooltipText = LocationTooltip;
					break;
				}
				case ESlateTransformComponent::Rotation:
				{
					TooltipText = RotationTooltip;
					break;
				}
				case ESlateTransformComponent::Scale:
				{
					TooltipText = ScaleTooltip;
					break;
				}
			}

			return SNew(SButton)
				.IsEnabled(true)
				.ContentPadding(0.f)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked_Lambda([OnIsComponentRelativeChanged, OnGetIsComponentRelative, InComponent]() -> FReply
				{
					if(OnIsComponentRelativeChanged.IsBound())
					{
						bool bIsRelative = true;
						if(OnGetIsComponentRelative.IsBound())
						{
							bIsRelative = OnGetIsComponentRelative.Execute(InComponent);
						}

						if(FSlateApplication::Get().GetModifierKeys().IsShiftDown() ||
							FSlateApplication::Get().GetModifierKeys().IsControlDown() ||
							FSlateApplication::Get().GetModifierKeys().IsAltDown())
						{
							OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Location, !bIsRelative);
							OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Rotation, !bIsRelative);
							OnIsComponentRelativeChanged.Execute(ESlateTransformComponent::Scale, !bIsRelative);
						}
						else
						{
							OnIsComponentRelativeChanged.Execute(InComponent, !bIsRelative);
						}
					}					
					return FReply::Handled();
				})
				.ToolTipText(TooltipText)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor(FSlateColor::UseStyle())
				.Content()
				[
					SNew(SImage)
					.Image_Lambda([OnGetIsComponentRelative, InComponent]() -> const FSlateBrush*
					{
						bool bIsRelative = true;
						if(OnGetIsComponentRelative.IsBound())
						{
							bIsRelative = OnGetIsComponentRelative.Execute(InComponent);
						}
						return GetRelativeWorldIcon(bIsRelative).GetIcon();
					})
					.ColorAndOpacity(FSlateColor::UseForeground())
				];
		}
		return SNullWidget::NullWidget;
	}

	/**
	 * Gather the icon for relative world
	 */
	static const FSlateIcon& GetRelativeWorldIcon(bool bRelative)
	{
		if (bRelative)
		{
			static FName LocalIconName("Icons.Transform");
			static FSlateIcon LocalIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), LocalIconName);
			return LocalIcon;
		}
		
		static FName WorldIconName("EditorViewport.RelativeCoordinateSystem_World");
		static FSlateIcon WorldIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), WorldIconName);
		return WorldIcon;
	}

#if WITH_EDITOR

	static IDetailGroup& ConstructDetailGroup(IDetailCategoryBuilder& InBuilder, FName GroupName, const FText& LocalizedDisplayName)
	{
		return InBuilder.AddGroup(GroupName, LocalizedDisplayName, false, true);
	}

	static IDetailGroup& ConstructDetailGroup(IDetailChildrenBuilder& InBuilder, FName GroupName, const FText& LocalizedDisplayName)
	{
		return InBuilder.AddGroup(GroupName, LocalizedDisplayName);
	}

	static void ConfigureComponentWidgetRow(
		FDetailWidgetRow& WidgetRow, 
		ESlateTransformComponent::Type InComponent,
		typename SAdvancedTransformInputBox<TransformType>::FArguments WidgetArgs)
	{
		if(WidgetArgs._OnCopyToClipboard.IsBound() && WidgetArgs._OnPasteFromClipboard.IsBound())
		{
			WidgetRow
			.CopyAction(FUIAction(
				FExecuteAction::CreateLambda([WidgetArgs, InComponent]()
				{
					WidgetArgs._OnCopyToClipboard.ExecuteIfBound(InComponent);
				}),
				FCanExecuteAction())
			);

			WidgetRow.PasteAction(FUIAction(
			FExecuteAction::CreateLambda([WidgetArgs, InComponent]()
			{
				WidgetArgs._OnPasteFromClipboard.ExecuteIfBound(InComponent);
			}),
			FCanExecuteAction::CreateLambda([WidgetArgs]() -> bool
			{
				return WidgetArgs._IsEnabled.Get();
			})));
		}

		if(WidgetArgs._OnResetToDefault.IsBound() && WidgetArgs._DiffersFromDefault.IsBound())
		{
			WidgetRow.OverrideResetToDefault(FResetToDefaultOverride::Create(
				TAttribute<bool>::CreateLambda([WidgetArgs, InComponent]() -> bool
				{
					if(!WidgetArgs._IsEnabled.Get())
					{
						return false;
					}
					return WidgetArgs._DiffersFromDefault.Execute(InComponent);
				}),
				FSimpleDelegate::CreateLambda([WidgetArgs, InComponent]()
				{
					WidgetArgs._OnResetToDefault.Execute(InComponent);
				})
			));
		}
			
		if(InComponent != ESlateTransformComponent::Max)
		{
			WidgetRow
			.NameContent()
			.HAlign(HAlign_Fill)
			[
				SAdvancedTransformInputBox<TransformType>::ConstructLabel(WidgetArgs, InComponent)
			]
			.ValueContent()
			.MinDesiredWidth(375.f)
			.MaxDesiredWidth(375.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SAdvancedTransformInputBox<TransformType>::ConstructWidget(WidgetArgs, InComponent)
				]
			];
		}			
	};

	static void ConfigureHeader(
		FDetailWidgetRow& HeaderRow,
		const FText& InLabel,
		const FText& InTooltip,
		typename SAdvancedTransformInputBox<TransformType>::FArguments WidgetArgs,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>()
	)
	{
		ConfigureComponentWidgetRow(HeaderRow, ESlateTransformComponent::Max, WidgetArgs);

		if(!NameContent.IsValid())
		{
			SAssignNew(NameContent, STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(InLabel)
			.ToolTipText(InTooltip);
		}
		
		HeaderRow
		.Visibility(WidgetArgs._Visibility)
		.NameContent()
		[
			NameContent.ToSharedRef()
		];
	}

	template<typename BuilderType = IDetailCategoryBuilder>
	static FDetailWidgetRow& ConstructGroupedTransformRows(
		BuilderType& InBuilder,
		const FText& InLabel,
		const FText& InTooltip,
		typename SAdvancedTransformInputBox<TransformType>::FArguments WidgetArgs,
		TSharedPtr<SWidget> NameContent = TSharedPtr<SWidget>()
		)
	{
		IDetailGroup& Group = ConstructDetailGroup(InBuilder, *InLabel.ToString(), InLabel);
		FDetailWidgetRow& HeaderRow = Group.HeaderRow();
		ConfigureHeader(HeaderRow, InLabel, InTooltip, WidgetArgs, NameContent);

		if(WidgetArgs._ConstructLocation)
		{
			FDetailWidgetRow& WidgetRow = Group.AddWidgetRow();
			ConfigureComponentWidgetRow(WidgetRow, ESlateTransformComponent::Location, WidgetArgs);
		}

		if(WidgetArgs._ConstructRotation)
		{
			FDetailWidgetRow& WidgetRow = Group.AddWidgetRow();
			ConfigureComponentWidgetRow(WidgetRow, ESlateTransformComponent::Rotation, WidgetArgs);
		}

		if(WidgetArgs._ConstructScale)
		{
			FDetailWidgetRow& WidgetRow = Group.AddWidgetRow();
			ConfigureComponentWidgetRow(WidgetRow, ESlateTransformComponent::Scale, WidgetArgs);
		}

		return HeaderRow;
	}

#endif

	static const TArray<FText>& GetRotationRepresentationLabels()
	{
		static TArray<FText> Labels;
		if(Labels.IsEmpty())
		{
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerXYZ", "Euler XYZ"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerXZY", "Euler XZY"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerYXZ", "Euler YXZ"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerYZX", "Euler YZX"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerZXY", "Euler ZXY"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "EulerZYX", "Euler ZYX"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "Rotator", "Rotator"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "Quaternion", "Quaternion"));
			Labels.Add(NSLOCTEXT("SAdvancedTransformInputBox", "AxisAndAngle", "Axis And Angle"));
		}
		return Labels;
	}

	static void ConstrainScale(
		int32 ComponentIndex,
		UE::Math::TVector<NumericType> OldValue,
		UE::Math::TVector<NumericType>& NewValue,
		TAttribute<bool> ConstrainComponents)
	{
		if(!ConstrainComponents.Get())
		{
			return;
		}

		if(FMath::IsNearlyZero(OldValue[ComponentIndex]) || FMath::IsNearlyZero(NewValue[ComponentIndex]))
		{
			return;
		}

		const NumericType Ratio = NewValue[ComponentIndex] / OldValue[ComponentIndex];
		NewValue = OldValue * Ratio;
	}

	static TOptional<NumericType> GetNumericValueFromTransform(
		const TransformType& Transform,
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent)
	{
		switch(Component)
		{
			case ESlateTransformComponent::Location:
			{
				switch(SubComponent)
				{
					case ESlateTransformSubComponent::X:
						return Transform.GetLocation().X;
					case ESlateTransformSubComponent::Y:
						return Transform.GetLocation().Y;
					case ESlateTransformSubComponent::Z:
						return Transform.GetLocation().Z;
				}
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				switch(Representation)
				{
					case ESlateRotationRepresentation::EulerXYZ:
					case ESlateRotationRepresentation::EulerXZY:
					case ESlateRotationRepresentation::EulerYXZ:
					case ESlateRotationRepresentation::EulerYZX:
					case ESlateRotationRepresentation::EulerZXY:
					case ESlateRotationRepresentation::EulerZYX:
					{
						const int32 RotationOrder = int32(Representation) - int32(ESlateRotationRepresentation::EulerXYZ);
						const FVector Euler = AnimationCore::EulerFromQuat(Transform.GetRotation(), EEulerRotationOrder(RotationOrder), true);
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
							{
								return Euler.X;
							}
							case ESlateTransformSubComponent::Y:
							{
								return Euler.Y;
							}
							case ESlateTransformSubComponent::Z:
							{
								return Euler.Z;
							}
						}
						break;
					}
					case ESlateRotationRepresentation::Rotator:
					{
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::Roll:
								return Transform.Rotator().Roll;
							case ESlateTransformSubComponent::Pitch:
								return Transform.Rotator().Pitch;
							case ESlateTransformSubComponent::Yaw:
								return Transform.Rotator().Yaw;
						}
						break;
					}
					case ESlateRotationRepresentation::Quaternion:
					{
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
								return Transform.GetRotation().X;
							case ESlateTransformSubComponent::Y:
								return Transform.GetRotation().Y;
							case ESlateTransformSubComponent::Z:
								return Transform.GetRotation().Z;
							case ESlateTransformSubComponent::W:
								return Transform.GetRotation().W;
						}
						break;
					}
					case ESlateRotationRepresentation::AxisAndAngle:
					{
						FVector Axis;
						NumericType Angle;
						Transform.GetRotation().ToAxisAndAngle(Axis, Angle);

						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
								return Axis.X;
							case ESlateTransformSubComponent::Y:
								return Axis.Y;
							case ESlateTransformSubComponent::Z:
								return Axis.Z;
							case ESlateTransformSubComponent::Angle:
								return Angle;
						}
						break;
					}
				}
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				switch(SubComponent)
				{
					case ESlateTransformSubComponent::X:
						return Transform.GetScale3D().X;
					case ESlateTransformSubComponent::Y:
						return Transform.GetScale3D().Y;
					case ESlateTransformSubComponent::Z:
						return Transform.GetScale3D().Z;
				}
				break;
			}
		}
		return TOptional<NumericType>();
	}

	static void ApplyNumericValueChange(
		TransformType& Transform,
		NumericType Value,
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent)
	{
		switch(Component)
		{
			case ESlateTransformComponent::Location:
			{
				UE::Math::TVector<NumericType> Location = Transform.GetLocation();
				switch(SubComponent)
				{
					case ESlateTransformSubComponent::X:
					{
						Location.X = Value;
						break;
					}
					case ESlateTransformSubComponent::Y:
					{
						Location.Y = Value;
						break;
					}
					case ESlateTransformSubComponent::Z:
					{
						Location.Z = Value;
						break;
					}
				}
				Transform.SetLocation(Location);
				break;
			}
			case ESlateTransformComponent::Rotation:
			{
				switch(Representation)
				{
					case ESlateRotationRepresentation::EulerXYZ:
					case ESlateRotationRepresentation::EulerXZY:
					case ESlateRotationRepresentation::EulerYXZ:
					case ESlateRotationRepresentation::EulerYZX:
					case ESlateRotationRepresentation::EulerZXY:
					case ESlateRotationRepresentation::EulerZYX:
					{
						const int32 RotationOrder = int32(Representation) - int32(ESlateRotationRepresentation::EulerXYZ);
						FVector Euler = AnimationCore::EulerFromQuat(Transform.GetRotation(), EEulerRotationOrder(RotationOrder));
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
							{
								Euler.X = Value;
								break;
							}
							case ESlateTransformSubComponent::Y:
							{
								Euler.Y = Value;
								break;
							}
							case ESlateTransformSubComponent::Z:
							{
								Euler.Z = Value;
								break;
							}
						}
						Transform.SetRotation(AnimationCore::QuatFromEuler(Euler, EEulerRotationOrder(RotationOrder), true));
						break;
					}
					case ESlateRotationRepresentation::Rotator:
					{
						FRotator Rotator = Transform.Rotator();
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::Roll:
							{
								Rotator.Roll = Value;
								break;
							}
							case ESlateTransformSubComponent::Pitch:
							{
								Rotator.Pitch = Value;
								break;
							}
							case ESlateTransformSubComponent::Yaw:
							{
								Rotator.Yaw = Value;
								break;
							}
						}
						Transform = TransformType(Rotator, Transform.GetLocation(), Transform.GetScale3D());
						break;
					}
					case ESlateRotationRepresentation::Quaternion:
					{
						FQuat Rotation = Transform.GetRotation();
						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
							{
								Rotation.X = Value;
								break;
							}
							case ESlateTransformSubComponent::Y:
							{
								Rotation.Y = Value;
								break;
							}
							case ESlateTransformSubComponent::Z:
							{
								Rotation.Z = Value;
								break;
							}
							case ESlateTransformSubComponent::W:
							{
								Rotation.W = Value;
								break;
							}
						}
						Rotation.Normalize();
						Transform.SetRotation(Rotation);
						break;
					}
					case ESlateRotationRepresentation::AxisAndAngle:
					{
						FVector Axis;
						NumericType Angle;
						Transform.GetRotation().ToAxisAndAngle(Axis, Angle);

						switch(SubComponent)
						{
							case ESlateTransformSubComponent::X:
							{
								Axis.X = Value;
								break;
							}
							case ESlateTransformSubComponent::Y:
							{
								Axis.Y = Value;
								break;
							}
							case ESlateTransformSubComponent::Z:
							{
								Axis.Z = Value;
								break;
							}
							case ESlateTransformSubComponent::Angle:
							{
								Angle = Value;
								break;
							}
						}
						Transform.SetRotation(FQuat(Axis, Angle));
						break;
					}
				}
				break;
			}
			case ESlateTransformComponent::Scale:
			{
				UE::Math::TVector<NumericType> Scale3D = Transform.GetScale3D();
				switch(SubComponent)
				{
					case ESlateTransformSubComponent::X:
					{
						Scale3D.X = Value;
						break;
					}
					case ESlateTransformSubComponent::Y:
					{
						Scale3D.Y = Value;
						break;
					}
					case ESlateTransformSubComponent::Z:
					{
						Scale3D.Z = Value;
						break;
					}
				}
				Transform.SetScale3D(Scale3D);
				break;
			}
		}
	}
};
